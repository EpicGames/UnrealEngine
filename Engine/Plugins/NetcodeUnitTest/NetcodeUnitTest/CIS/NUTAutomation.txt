-nosteam -ini:Engine:[Voice]:bEnabled=False
-ExecCmds="automation quit, automation runtests IPClient+WebSocketClient+VMReflection+PartialInitialCheck+FTextCrash+DirCommand+LoginURLCrash+LoginURLAlloc+LogOverwrite+FastClientSeamlessTravel+MidJoinSeamlessTravel+EquipWeaponCheck"
