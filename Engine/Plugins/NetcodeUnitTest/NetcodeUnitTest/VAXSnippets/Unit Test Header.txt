Title: Unit Test Header
Shortcut:
Description: Creates the header file for a basic unit test.

Source:
// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ClientUnitTest.h"

#include "$FILE_BASE$.generated.h"


/**
 * @todo #JohnB: $end$
 */
UCLASS()
class U$FILE_BASE$ : public UClientUnitTest
{
	GENERATED_UCLASS_BODY()

public:
	virtual void InitializeEnvironmentSettings() override;

	virtual void ExecuteClientUnitTest() override;

	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;
};

