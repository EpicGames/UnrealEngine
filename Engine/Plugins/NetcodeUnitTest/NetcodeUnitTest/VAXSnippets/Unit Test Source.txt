Title: Unit Test Source
Shortcut:
Description: Creates the source file for a basic unit test.

Source:
// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTests/$FILE_BASE$.h"

#include "UnitTestEnvironment.h"
#include "MinimalClient.h"


/**
 * U$FILE_BASE$
 */

U$FILE_BASE$::U$FILE_BASE$(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UnitTestName = TEXT("$UnitTestName$");
	UnitTestType = TEXT("$UnitTestType$");

	UnitTestDate = FDateTime($YEAR$, $MONTH$, $DAY$);

	bWorkInProgress = true;

	// @todo #JohnBExploitCL: Bugtracking/changelist notes

	ExpectedResult.Add(TEXT("$TargetGame$"), EUnitTestVerification::VerifiedNotFixed);

	UnitTestTimeout = 60;


	SetFlags<EUnitTestFlags::LaunchServer | $end$,>();
}

void U$FILE_BASE$::InitializeEnvironmentSettings()
{
	BaseServerURL = UnitEnv->GetDefaultMap(UnitTestFlags);
	BaseServerParameters = UnitEnv->GetDefaultServerParameters();

	UnitEnv->InitializeUnitTasks();
}

void U$FILE_BASE$::ExecuteClientUnitTest()
{
	// @todo #JohnB
}

void U$FILE_BASE$::NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines)
{
	Super::NotifyProcessLog(InProcess, InLogLines);

	if (InProcess.HasSameObject(ServerHandle.Pin().Get()))
	{
		for (auto CurLine : InLogLines)
		{
			// @todo #JohnB
		}
	}
}


