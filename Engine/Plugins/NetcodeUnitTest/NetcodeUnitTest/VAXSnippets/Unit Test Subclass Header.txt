Title: Unit Test Subclass Header
Shortcut:
Description: Creates a header for a unit test subclass.

Source:
// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnitTests/$SubclassUnitTest$.h"

#include "$FILE_BASE$.generated.h"

/**
 * @todo #JohnB: $end$
 */
UCLASS()
class U$FILE_BASE$ : public U$SubclassUnitTest$
{
	GENERATED_UCLASS_BODY()
};
