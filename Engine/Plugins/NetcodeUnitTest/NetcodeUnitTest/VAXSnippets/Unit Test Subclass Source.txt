Title: Unit Test Subclass Source
Shortcut:
Description: Creates the source file for a unit test subclass.

Source:
// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTests/$FILE_BASE$.h"

#include "UnitTestEnvironment.h"

/**
 * U$FILE_BASE$
 */

U$FILE_BASE$::U$FILE_BASE$(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UnitTestName = TEXT("$UnitTestName$");

	UnitTestDate = FDateTime($YEAR$, $MONTH$, $DAY$);

	UnitTestBugTrackIDs.Empty();

	bWorkInProgress = true;

	// @todo #JohnBExploitCL: Bugtracking/changelist notes

	ExpectedResult.Add(TEXT("$TargetGame$"), EUnitTestVerification::VerifiedNotFixed);
}
